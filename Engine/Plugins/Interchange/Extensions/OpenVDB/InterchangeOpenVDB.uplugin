{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Interchange OpenVDB",
	"Description": "Allows translation of OpenVDB files via the Interchange framework",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Hidden": true,
	"IsExperimentalVersion": true,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux"
	],
	"Modules": [
		{
			"Name": "InterchangeOpenVDBImport",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Interchange",
			"Enabled": true
		}
	]
}