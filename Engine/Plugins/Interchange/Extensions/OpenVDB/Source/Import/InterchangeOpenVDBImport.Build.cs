// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class InterchangeOpenVDBImport : ModuleRules
	{
		public InterchangeOpenVDBImport(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"InterchangeCore", 		// For UInterchangeBaseNode, base translator
					"InterchangeImport", 	// For the translator settings, pipeline interface, etc.
					"InterchangeNodes", 	// For the actual volume nodes
					"InterchangeEngine", 	// For UInterchangeManager::GetInterchangeManager
					"SparseVolumeTexture",	// For the legacy OpenVDB importer reference, that actually uses the OpenVDB SDK
				}
			);
		}
	}
}
