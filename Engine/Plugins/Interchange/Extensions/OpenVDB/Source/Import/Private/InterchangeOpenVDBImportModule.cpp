// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeOpenVDBImportModule.h"

#include "InterchangeOpenVDBImportLog.h"
#include "InterchangeOpenVDBTranslator.h"

#include "Engine/Engine.h"
#include "InterchangeManager.h"
#include "InterchangeProjectSettings.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogInterchangeOpenVDBImport);

void FInterchangeOpenVDBImportModule::StartupModule()
{
	using namespace UE::Interchange;

	auto RegisterItems = [this]()
	{
// Editor and OpenVDB-only because we use GetOpenVDBGridInfo and ConvertOpenVDBToSparseVolumeTexture
// which use OpenVDB and are in an Editor-only module
#if WITH_EDITOR && OPENVDB_AVAILABLE
		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		InterchangeManager.RegisterTranslator(UInterchangeOpenVDBTranslator::StaticClass());

		TSoftClassPtr<UInterchangeTranslatorBase> TranslatorClassPath = TSoftClassPtr<UInterchangeTranslatorBase>(
			UInterchangeOpenVDBTranslator::StaticClass()
		);

		FInterchangeTranslatorPipelines TranslatorPipelines;
		TranslatorPipelines.Translator = TranslatorClassPath;
		TranslatorPipelines.Pipelines.Add(
			FSoftObjectPath{TEXT("/Interchange/Pipelines/DefaultSparseVolumeTexturePipeline.DefaultSparseVolumeTexturePipeline")}
		);

		// Don't go through FInterchangeProjectSettingsUtils::GetMutableDefaultImportSettings as we'll need a member of the actual
		// FInterchangeContentImportSettings struct anyway, and this is likely safer than casting the struct pointer
		UInterchangeProjectSettings* ProjectSettings = GetMutableDefault<UInterchangeProjectSettings>();
		if (!ProjectSettings)
		{
			return;
		}

		// Scene import pipeline stacks
		{
			FInterchangeImportSettings& SceneImportSettings = ProjectSettings->SceneImportSettings;
			const FName DefaultPipelineStack = SceneImportSettings.DefaultPipelineStack;

			if (SceneImportSettings.PipelineStacks.Contains(DefaultPipelineStack))
			{
				FInterchangePipelineStack& PipelineStack = SceneImportSettings.PipelineStacks[DefaultPipelineStack];
				PipelineStack.PerTranslatorPipelines.Add(TranslatorPipelines);

				UE_LOGF(LogInterchangeOpenVDBImport, Log, "InterchangeOpenVDB's pipelines added to default Scene pipeline stack, %ls", *DefaultPipelineStack.ToString());
			}
			else
			{
				UE_LOGF(LogInterchangeOpenVDBImport, Warning, "Cannot find Interchange default Scene's stack to add pipelines. InterchangeOpenVDB level import may not work properly.");
			}
		}

		// Asset and Texture import pipeline stacks
		{
			FInterchangeContentImportSettings& AssetImportSettings = ProjectSettings->ContentImportSettings;

			const FName DefaultPipelineStack = AssetImportSettings.DefaultPipelineStack;
			if (AssetImportSettings.PipelineStacks.Contains(DefaultPipelineStack))
			{
				FInterchangePipelineStack& AssetStack = AssetImportSettings.PipelineStacks[DefaultPipelineStack];
				AssetStack.PerTranslatorPipelines.Add(TranslatorPipelines);

				UE_LOGF(LogInterchangeOpenVDBImport, Log, "InterchangeOpenVDB's pipelines added to default Asset pipeline stack, %ls", *DefaultPipelineStack.ToString());
			}
			else
			{
				UE_LOGF(LogInterchangeOpenVDBImport, Warning, "Cannot find Interchange default Asset's stack to add pipelines. InterchangeOpenVDB asset import may not work properly.");
			}

			if (AssetImportSettings.DefaultPipelineStackOverride.Contains(EInterchangeTranslatorAssetType::Textures))
			{
				const FName DefaultTextureStack = AssetImportSettings.DefaultPipelineStackOverride[EInterchangeTranslatorAssetType::Textures];
				if (AssetImportSettings.PipelineStacks.Contains(DefaultTextureStack))
				{
					FInterchangePipelineStack& TextureStack = AssetImportSettings.PipelineStacks[DefaultTextureStack];
					TextureStack.PerTranslatorPipelines.Add(TranslatorPipelines);

					// Asset import show dialog override (otherwise it doesn't show the import options dialog as OpenVDBs are Texture type assets)
					{
						FInterchangeDialogOverride& DialogOverrides = AssetImportSettings.ShowImportDialogOverride.FindOrAdd(
							EInterchangeTranslatorAssetType::Textures
						);

						FInterchangePerTranslatorDialogOverride ImportDialogOverride;
						ImportDialogOverride.Translator = TranslatorClassPath;
						ImportDialogOverride.bShowImportDialog = true;
						ImportDialogOverride.bShowReimportDialog = true;
						DialogOverrides.PerTranslatorImportDialogOverride.Add(ImportDialogOverride);
					}
				}
				else
				{
					UE_LOGF(LogInterchangeOpenVDBImport, Warning, "Cannot find Interchange default Textures's stack to add pipelines. InterchangeOpenVDB Texture import may not work properly.");
				}
			}
		}
#endif
	};

	if (GEngine)
	{
		RegisterItems();
	}
	else
	{
		PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterItems);
	}
}

void FInterchangeOpenVDBImportModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
}

IMPLEMENT_MODULE(FInterchangeOpenVDBImportModule, InterchangeOpenVDBImport)
