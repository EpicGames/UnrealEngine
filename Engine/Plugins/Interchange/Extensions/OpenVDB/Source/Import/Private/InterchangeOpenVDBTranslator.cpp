// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeOpenVDBTranslator.h"

#include "InterchangeManager.h"
#include "InterchangeOpenVDBImportLog.h"
#include "InterchangeSceneNode.h"
#include "InterchangeVolumeNode.h"
#include "Usd/InterchangeUsdDefinitions.h"
#include "Volume/InterchangeVolumeTranslatorSettings.h"

#include "Async/ParallelFor.h"
#include "HAL/IConsoleManager.h"

#if WITH_EDITOR
#include "OpenVDBImportOptions.h"
#include "SparseVolumeTextureOpenVDBUtility.h"
#endif	  // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeOpenVDBTranslator)

#define LOCTEXT_NAMESPACE "InterchangeOpenVDBTranslator"

static int32 GMaxParallelFileReads = 16;
static FAutoConsoleVariableRef CVarNumPerPrimLocks(
	TEXT("Interchange.FeatureFlags.Import.OpenVDB.NumFileReadLocks"),
	GMaxParallelFileReads,
	TEXT("Maximum number of .vdb files that can be read concurrently. This uses system mutexes, which are limited and vary depending on your system."
	),
	ECVF_ReadOnly	 // We'll keep a static array of these for simplicity, so changing it during the session would be annoying to handle
);

namespace UE::InterchangeOpenVDBTranslator::Private
{
#if WITH_EDITOR
	struct FOpenVDBFileInfo
	{
		TArray64<uint8> FileBytes;
		TArray<FOpenVDBGridInfo> GridInfo;

	private:
		friend class UInterchangeOpenVDBTranslatorImpl;

		// Index of our lock within the Impl's FileLocks. Having multiple locks allows concurrent file reads
		int32 LockIndex = INDEX_NONE;

		// We set this to true if we failed to load/open this file, so that we don't try again later and violate the
		// assumption that the FileInfos are going to be read-only after they're returned from GetOrLoadFileInfo
		bool bFailed = false;
	};

	EVolumeGridElementType GridTypeToInterchangeGridType(EOpenVDBGridType Type)
	{
		switch (Type)
		{
			case EOpenVDBGridType::Half:
			case EOpenVDBGridType::Half2:
			case EOpenVDBGridType::Half3:
			case EOpenVDBGridType::Half4:
			{
				return EVolumeGridElementType::Half;
			}
			case EOpenVDBGridType::Float:
			case EOpenVDBGridType::Float2:
			case EOpenVDBGridType::Float3:
			case EOpenVDBGridType::Float4:
			{
				return EVolumeGridElementType::Float;
			}
			case EOpenVDBGridType::Double:
			case EOpenVDBGridType::Double2:
			case EOpenVDBGridType::Double3:
			case EOpenVDBGridType::Double4:
			{
				return EVolumeGridElementType::Double;
			}
			default:
			{
				return EVolumeGridElementType::Unknown;
			}
		}
	}

	class UInterchangeOpenVDBTranslatorImpl
	{
	public:
		UInterchangeOpenVDBTranslatorImpl()
		{
			FileLocks = new FRWLock[GMaxParallelFileReads]();
		}

		~UInterchangeOpenVDBTranslatorImpl()
		{
			delete[] FileLocks;
		}

		UInterchangeOpenVDBTranslatorImpl(const UInterchangeOpenVDBTranslatorImpl& Other) = delete;
		UInterchangeOpenVDBTranslatorImpl& operator=(const UInterchangeOpenVDBTranslatorImpl& Other) = delete;

	public:
		void ClearFileInfoEntries()
		{
			FWriteScopeLock WriteLock(FileNameToInfoLock);

			FileNameToInfo.Reset();
		}

		void EnsureFileInfoEntries(const TArray<FString>& Filenames)
		{
			FWriteScopeLock WriteLock(FileNameToInfoLock);

			for (const FString& Filename : Filenames)
			{
				if (FileNameToInfo.Contains(Filename))
				{
					continue;
				}

				TUniquePtr<FOpenVDBFileInfo> NewInfo = MakeUnique<FOpenVDBFileInfo>();
				NewInfo->LockIndex = NextFileLockIndex++;

				FileNameToInfo.Add(Filename, MoveTemp(NewInfo));
			}
		}

		const FOpenVDBFileInfo* GetOrLoadFileInfo(const FString& Filename)
		{
			FOpenVDBFileInfo* Info = nullptr;
			{
				FReadScopeLock ReadLock(FileNameToInfoLock);

				if (TUniquePtr<FOpenVDBFileInfo>* FoundInfo = FileNameToInfo.Find(Filename))
				{
					Info = FoundInfo->Get();
				}
			}
			if (!Info)
			{
				return nullptr;
			}

			// Load on-demand if we need to
			//
			// Only try to do this once, and set bFailed to true if we failed, so that we can assume
			// that nothing ever writes to this after we first try this on-demand load. With this assumption,
			// we don't need to use read locks everywhere
			{
				FWriteScopeLock WriteLock(FileLocks[Info->LockIndex % GMaxParallelFileReads]);

				if (!Info->bFailed && Info->FileBytes.Num() == 0)
				{
					if (!FPaths::FileExists(Filename))
					{
						UE_LOGF(LogInterchangeOpenVDBImport, Error, "OpenVDB file could not be found: %ls", *Filename);
						Info->bFailed = true;
						return nullptr;
					}

					if (!FFileHelper::LoadFileToArray(Info->FileBytes, *Filename))
					{
						UE_LOGF(LogInterchangeOpenVDBImport, Error, "OpenVDB file could not be loaded: %ls", *Filename);
						Info->bFailed = true;
						return nullptr;
					}

					const bool bCreateStrings = true;
					if (!GetOpenVDBGridInfo(Info->FileBytes, bCreateStrings, &Info->GridInfo))
					{
						UE_LOGF(LogInterchangeOpenVDBImport, Error, "Failed to read OpenVDB file: %ls", *Filename);
						Info->bFailed = true;
						return nullptr;
					}
				}
			}

			return Info;
		}

		const TMap<FString, TUniquePtr<FOpenVDBFileInfo>>& GetAllFileInfo() const
		{
			return FileNameToInfo;
		}

	private:
		// We don't need this when translating files directly, as we can ensure FileNameToInfo only receives
		// new entries on the game thread. If this translator is used indirectly however (i.e. through another
		// translator like USDTranslator) we may be asked to return payload data for a file we never seen
		// before and may need to open on-demand. Since payload retrieval can be multi-threaded, we may
		// have multiple threads trying to add entries to FileNameToInfo at the same time, which is what
		// this lock is for.
		FRWLock FileNameToInfoLock;
		TMap<FString, TUniquePtr<FOpenVDBFileInfo>> FileNameToInfo;

		FRWLock* FileLocks = nullptr;
		int32 NextFileLockIndex = 0;
	};
#else
	class UInterchangeOpenVDBTranslatorImpl
	{
	};
#endif	  // WITH_EDITOR
}	 // namespace UE::InterchangeOpenVDBTranslator::Private

UInterchangeOpenVDBTranslator::UInterchangeOpenVDBTranslator()
	: Impl(MakeUnique<UE::InterchangeOpenVDBTranslator::Private::UInterchangeOpenVDBTranslatorImpl>())
{
}

EInterchangeTranslatorType UInterchangeOpenVDBTranslator::GetTranslatorType() const
{
	return EInterchangeTranslatorType::Assets;
}

EInterchangeTranslatorAssetType UInterchangeOpenVDBTranslator::GetSupportedAssetTypes() const
{
	return EInterchangeTranslatorAssetType::Textures;
}

bool UInterchangeOpenVDBTranslator::CanImportSourceData(const UInterchangeSourceData* InSourceData) const
{
	// Ignore the cvar's effect from GetSupportedFormats() in case we're being used by the USD translator
	// to import USD+OpenVDB files (it will add this additional context object)
	UObject* ContextObject = InSourceData->GetContextObjectByTag(UE::Interchange::USD::USDContextTag);
	if (ContextObject)
	{
		const bool bIncludeDot = false;
		const FString Extension = FPaths::GetExtension(InSourceData->GetFilename(), bIncludeDot).ToLower();
		if (Extension == TEXT("vdb"))
		{
			return true;
		}
	}

	return Super::CanImportSourceData(InSourceData);
}

TArray<FString> UInterchangeOpenVDBTranslator::GetSupportedFormats() const
{
	TArray<FString> Extensions;

	// We don't advertise support for .vdb files here. The UInterchangeOpenVDBTranslator is exclusively used
	// by the UInterchangeUSDTranslator for now, and shouldn't import .vdb files directly
	// Extensions.Add(TEXT("vdb;OpenVDB files"));

	return Extensions;
}

bool UInterchangeOpenVDBTranslator::Translate(UInterchangeBaseNodeContainer& NodeContainer) const
{
#if WITH_EDITOR && OPENVDB_AVAILABLE
	using namespace UE::InterchangeOpenVDBTranslator::Private;

	// References:
	// - SparseVolumeTextureFactory.cpp, LoadOpenVDBPreviewData()

	UInterchangeOpenVDBTranslatorImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return false;
	}

	UInterchangeVolumeTranslatorSettings* Settings = Cast<UInterchangeVolumeTranslatorSettings>(GetSettings());
	if (!Settings)
	{
		return false;
	}

	// Get all relevant VDB filenames to import
	TArray<FString> Filenames;
	{
		const FString MainFilename = GetSourceData()->GetFilename();

		if (Settings->bTranslateAdjacentNumberedFiles)
		{
			Filenames = FindOpenVDBSequenceFileNames(MainFilename);
		}
		else
		{
			Filenames = {MainFilename};
		}
	}

	// Group up multiple VDB files in a single animation ID, so that the SVT pipeline makes one animated SVT factory node out of the group.
	//
	// We try using the ID specified in the settings first, because it's handy to be able to drive this via the USD translator as
	// the AnimationID will make it's way into the volume / grid node UIDs, which is cumbersome to patch up afterwards
	FString AnimationID = Settings->AnimationID;
	if (AnimationID.IsEmpty() && Filenames.Num() > 1)
	{
		FSHA1 SHA1;
		for (const FString& Filename : Filenames)
		{
			SHA1.UpdateWithString(*Filename, Filename.Len());
		}
		SHA1.Final();

		FSHAHash FilePathHash;
		SHA1.GetHash(&FilePathHash.Hash[0]);
		AnimationID = FilePathHash.ToString();
	}

	// Create all the entries we'll need in one thread
	ImplPtr->ClearFileInfoEntries();
	ImplPtr->EnsureFileInfoEntries(Filenames);

	// Collect all the frame info in parallel
	const int32 NumBatches = Filenames.Num();
	ParallelFor(
		NumBatches,
		[ImplPtr, &Filenames](int32 FilenameIndex)
		{
			const FString& Filename = Filenames[FilenameIndex];
			ImplPtr->GetOrLoadFileInfo(Filename);
		}
	);

	// Note that Filenames are sorted inside FindOpenVDBSequenceFileNames, so this is also our volume frame index into the animation,
	// if we do have any (e.g. filenames [tornado_23.vdb, tornado_47.vdb, tornado_77.vdb] --> indices [0, 1, 2])
	for (int32 FilenameIndex = 0; FilenameIndex < Filenames.Num(); ++FilenameIndex)
	{
		const FString& Filename = Filenames[FilenameIndex];
		const FOpenVDBFileInfo* FileInfo = ImplPtr->GetOrLoadFileInfo(Filenames[FilenameIndex]);
		if (!FileInfo)
		{
			continue;
		}

		const static FString VolumePrefix = TEXT("\\Volume\\");
		const FString BaseFilename = FPaths::GetBaseFilename(Filename);

		// Full filename helps prevent name collisions.
		// AnimationID because the same volume may be used for different animations in the same import (e.g. via USD). In that case,
		// we want to emit a factory node for each animation, but it is nice to retain the correspondence between translated node
		// and factory node UIDs of just having the extra factory node prefix. That directly implies we need a separate volume node
		// for each animation ID, and so the animation ID must be part of the node UID
		const FString VolumeNodeUid = VolumePrefix + Filename + TEXT("\\") + AnimationID;

		// Generate node for this file
		UInterchangeVolumeNode* VolumeNode = nullptr;
		{
			const UInterchangeVolumeNode* Node = Cast<UInterchangeVolumeNode>(NodeContainer.GetNode(VolumeNodeUid));
			if (Node)
			{
				continue;
			}

			VolumeNode = NewObject<UInterchangeVolumeNode>(&NodeContainer);
			NodeContainer.SetupNode(VolumeNode, VolumeNodeUid, BaseFilename, EInterchangeNodeContainerType::TranslatedAsset);
			VolumeNode->SetAssetName(VolumeNodeUid);
			VolumeNode->SetCustomFileName(Filename);
			if (!AnimationID.IsEmpty())
			{
				VolumeNode->SetCustomAnimationID(AnimationID);
				VolumeNode->AddCustomFrameIndexInAnimation(FilenameIndex);
			}
		}

		// Generate nodes for each volume grid
		for (const FOpenVDBGridInfo& GridInfo : FileInfo->GridInfo)
		{
			const FString& GridName = GridInfo.Name;
			const FString GridNodeUid = VolumeNodeUid + TEXT("_") + GridName;

			const UInterchangeVolumeGridNode* Node = Cast<UInterchangeVolumeGridNode>(NodeContainer.GetNode(VolumeNodeUid));
			if (Node)
			{
				continue;
			}

			UInterchangeVolumeGridNode* GridNode = NewObject<UInterchangeVolumeGridNode>(&NodeContainer);
			NodeContainer.SetupNode(GridNode, GridNodeUid, GridName, EInterchangeNodeContainerType::TranslatedAsset, VolumeNodeUid);
			GridNode->SetCustomElementType(GridTypeToInterchangeGridType(GridInfo.Type));
			GridNode->SetCustomNumComponents(GridInfo.NumComponents);
			GridNode->SetCustomGridTransform(GridInfo.Transform);
			GridNode->SetCustomGridActiveAABBMin(GridInfo.VolumeActiveAABBMin);
			GridNode->SetCustomGridActiveAABBMax(GridInfo.VolumeActiveAABBMax);
			GridNode->SetCustomGridActiveDimensions(GridInfo.VolumeActiveDim);

			VolumeNode->AddCustomGridDependency(GridNodeUid);
		}
	}

	return true;
#else
	return false;
#endif	  // WITH_EDITOR && OPENVDB_AVAILABLE
}

void UInterchangeOpenVDBTranslator::ReleaseSource()
{
#if WITH_EDITOR
	UE::InterchangeOpenVDBTranslator::Private::UInterchangeOpenVDBTranslatorImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return;
	}

	ImplPtr->ClearFileInfoEntries();

	if (TranslatorSettings)
	{
		TranslatorSettings->ClearFlags(RF_Standalone);
		TranslatorSettings = nullptr;
	}
#endif	  // WITH_EDITOR
}

UInterchangeTranslatorSettings* UInterchangeOpenVDBTranslator::GetSettings() const
{
#if WITH_EDITOR
	using namespace UE::InterchangeOpenVDBTranslator::Private;

	UInterchangeOpenVDBTranslatorImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return nullptr;
	}

	if (!TranslatorSettings)
	{
		TranslatorSettings = DuplicateObject<UInterchangeVolumeTranslatorSettings>(
			UInterchangeVolumeTranslatorSettings::StaticClass()->GetDefaultObject<UInterchangeVolumeTranslatorSettings>(),
			GetTransientPackage()
		);
		TranslatorSettings->LoadSettings();
		TranslatorSettings->ClearFlags(RF_ArchetypeObject);
		TranslatorSettings->SetFlags(RF_Standalone);
		TranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);
	}
	return TranslatorSettings;
#else
	return nullptr;
#endif	  // WITH_EDITOR
}

void UInterchangeOpenVDBTranslator::SetSettings(const UInterchangeTranslatorSettings* InterchangeTranslatorSettings)
{
#if WITH_EDITOR
	using namespace UE::InterchangeOpenVDBTranslator::Private;

	UInterchangeOpenVDBTranslatorImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return;
	}

	if (TranslatorSettings)
	{
		TranslatorSettings->ClearFlags(RF_Standalone);
		TranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);
		TranslatorSettings = nullptr;
	}
	if (const UInterchangeVolumeTranslatorSettings* OpenVDBTranslatorSettings = Cast<UInterchangeVolumeTranslatorSettings>(
			InterchangeTranslatorSettings
		))
	{
		TranslatorSettings = DuplicateObject<UInterchangeVolumeTranslatorSettings>(OpenVDBTranslatorSettings, GetTransientPackage());
		TranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);
		TranslatorSettings->SetFlags(RF_Standalone);
	}
#endif	  // WITH_EDITOR
}

TOptional<UE::Interchange::FVolumePayloadData> UInterchangeOpenVDBTranslator::GetVolumePayloadData(
	const UE::Interchange::FVolumePayloadKey& PayloadKey
) const
{
#if WITH_EDITOR && OPENVDB_AVAILABLE
	using namespace UE::InterchangeOpenVDBTranslator::Private;

	UInterchangeOpenVDBTranslatorImpl* ImplPtr = Impl.Get();
	if (!ImplPtr)
	{
		return {};
	}

	// Note: We shouldn't need to lock FoundInfo's mutex to read from it after we got it:
	// We're not going to write to it after the initial load anyway
	const FOpenVDBFileInfo* FoundInfo = nullptr;
	{
		ImplPtr->EnsureFileInfoEntries({PayloadKey.FileName});

		FoundInfo = ImplPtr->GetOrLoadFileInfo(PayloadKey.FileName);
		if (!FoundInfo)
		{
			return {};
		}
	}

	// Convert our custom struct to the one used by ConvertOpenVDBToSparseVolumeTexture
	FOpenVDBImportOptions ImportOptions;
	{
		ImportOptions.bIsSequence = PayloadKey.AssignmentInfo.bIsSequence;

		for (int32 TextureIndex = 0; TextureIndex < ImportOptions.Attributes.Num(); ++TextureIndex)
		{
			const UE::Interchange::Volume::FTextureInfo& InTextureInfo = PayloadKey.AssignmentInfo.Attributes[TextureIndex];
			FOpenVDBSparseVolumeAttributesDesc& OutTextureInfo = ImportOptions.Attributes[TextureIndex];

			static_assert((int)ESparseVolumeAttributesFormat::Unorm8 == (int)EInterchangeSparseVolumeTextureFormat::Unorm8);
			static_assert((int)ESparseVolumeAttributesFormat::Float16 == (int)EInterchangeSparseVolumeTextureFormat::Float16);
			static_assert((int)ESparseVolumeAttributesFormat::Float32 == (int)EInterchangeSparseVolumeTextureFormat::Float32);
			OutTextureInfo.Format = static_cast<ESparseVolumeAttributesFormat>(InTextureInfo.Format);

			for (int32 ChannelIndex = 0; ChannelIndex < OutTextureInfo.Mappings.Num(); ++ChannelIndex)
			{
				OutTextureInfo.Mappings[ChannelIndex].SourceComponentIndex = InTextureInfo.Mappings[ChannelIndex].SourceComponentIndex;
				OutTextureInfo.Mappings[ChannelIndex].SourceGridIndex = InTextureInfo.Mappings[ChannelIndex].SourceGridIndex;
			}
		}
	}

	UE::Interchange::FVolumePayloadData Result;
	bool bSucess = ConvertOpenVDBToSparseVolumeTexture(
		const_cast<TArray64<uint8>&>(FoundInfo->FileBytes),	   // This function needs non-const access but it shouldn't ever write to anything
		ImportOptions,
		PayloadKey.VolumeBoundsMin,
		Result.TextureData,
		Result.Transform
	);
	if (bSucess)
	{
		return Result;
	}
#endif	  // WITH_EDITOR && OPENVDB_AVAILABLE

	return {};
}

#undef LOCTEXT_NAMESPACE
