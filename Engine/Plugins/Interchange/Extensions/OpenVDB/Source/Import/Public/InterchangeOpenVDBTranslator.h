// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangeTranslatorBase.h"
#include "Volume/InterchangeVolumePayloadInterface.h"

#include "UObject/ObjectMacros.h"

#include "InterchangeOpenVDBTranslator.generated.h"

namespace UE::Interchange
{
	struct FVolumePayloadData;
	struct FVolumePayloadKey;
}
namespace UE::InterchangeOpenVDBTranslator::Private
{
	class UInterchangeOpenVDBTranslatorImpl;
}
class UInterchangeVolumeTranslatorSettings;

UCLASS(BlueprintType)
class UInterchangeOpenVDBTranslator
	: public UInterchangeTranslatorBase
	, public IInterchangeVolumePayloadInterface
{
	GENERATED_BODY()

public:
	UInterchangeOpenVDBTranslator();

	/** Begin UInterchangeTranslatorBase interface */
	virtual EInterchangeTranslatorType GetTranslatorType() const override;
	virtual EInterchangeTranslatorAssetType GetSupportedAssetTypes() const override;
	virtual bool CanImportSourceData(const UInterchangeSourceData* InSourceData) const override;
	virtual TArray<FString> GetSupportedFormats() const override;
	virtual bool Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const override;
	virtual void ReleaseSource() override;
	virtual UInterchangeTranslatorSettings* GetSettings() const override;
	virtual void SetSettings(const UInterchangeTranslatorSettings* InterchangeTranslatorSettings) override;
	/** End UInterchangeTranslatorBase interface */

	/** Begin Interchange payload interfaces */
	virtual TOptional<UE::Interchange::FVolumePayloadData> GetVolumePayloadData(const UE::Interchange::FVolumePayloadKey& PayloadKey) const override;
	/** End Interchange payload interfaces */

private:
	TUniquePtr<UE::InterchangeOpenVDBTranslator::Private::UInterchangeOpenVDBTranslatorImpl> Impl;

	UPROPERTY(Transient, DuplicateTransient)
	mutable TObjectPtr<UInterchangeVolumeTranslatorSettings> TranslatorSettings = nullptr;
};
